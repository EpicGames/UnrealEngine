// Copyright Epic Games, Inc. All Rights Reserved.

#include "InputBindingSettingsViewerModule.h"

#include "Framework/Docking/TabManager.h"
#include "ISettingsCategory.h"
#include "ISettingsContainer.h"
#include "ISettingsEditorModel.h"
#include "ISettingsEditorModule.h"
#include "ISettingsModule.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "InputBindingSettingsViewerModule"

const FLazyName FInputBindingSettingsViewerModule::SettingsTabName("InputBindingSettings");
const FLazyName FInputBindingSettingsViewerModule::ContainerName("InputBinding");

void FInputBindingSettingsViewerModule::ShowSettings(const FName& InCategoryName, const FName& InSectionName)
{
	FGlobalTabmanager::Get()->TryInvokeTab(FTabId(SettingsTabName));
	if (const ISettingsEditorModelPtr SettingsEditorModel = SettingsEditorModelWeak.Pin())
	{
		if (const ISettingsCategoryPtr Category = SettingsEditorModel->GetSettingsContainer()->GetCategory(InCategoryName))
		{
			SettingsEditorModel->SelectSection(Category->GetSection(InSectionName));
		}
	}
}

void FInputBindingSettingsViewerModule::StartupModule()
{
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->RegisterViewer(ContainerName, *this);
	}

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SettingsTabName, FOnSpawnTab::CreateRaw(this, &FInputBindingSettingsViewerModule::HandleSpawnSettingsTab))
		.SetDisplayName(LOCTEXT("InputBindingSettingsTabTitle", "Keyboard Shortcuts"))
		.SetMenuType(ETabSpawnerMenuType::Hidden)
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "SystemWideCommands.OpenKeyboardShortcuts"));
}

void FInputBindingSettingsViewerModule::ShutdownModule()
{
	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SettingsTabName);
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		SettingsModule->UnregisterViewer(ContainerName);
	}
}

bool FInputBindingSettingsViewerModule::SupportsDynamicReloading()
{
	return true;
}

TSharedRef<SDockTab> FInputBindingSettingsViewerModule::HandleSpawnSettingsTab(const FSpawnTabArgs& InSpawnTabArgs)
{
	TSharedRef<SWidget> SettingsEditor = SNullWidget::NullWidget;
	if (ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings"))
	{
		if (const ISettingsContainerPtr SettingsContainer = SettingsModule->GetContainer(ContainerName))
		{
			ISettingsEditorModule& SettingsEditorModule = FModuleManager::GetModuleChecked<ISettingsEditorModule>("SettingsEditor");
			const ISettingsEditorModelRef SettingsEditorModel = SettingsEditorModule.CreateModel(SettingsContainer.ToSharedRef());
			SettingsEditor = SettingsEditorModule.CreateEditor(SettingsEditorModel);
			SettingsEditorModelWeak = SettingsEditorModel;
		}
	}

	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SettingsEditor
		];
}

IMPLEMENT_MODULE(FInputBindingSettingsViewerModule, InputBindingSettingsViewer);

#undef LOCTEXT_NAMESPACE