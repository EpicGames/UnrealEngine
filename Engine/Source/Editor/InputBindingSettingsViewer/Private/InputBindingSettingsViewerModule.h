// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISettingsViewer.h"
#include "Modules/ModuleInterface.h"

class FSpawnTabArgs;
class ISettingsEditorModel;
class SDockTab;

/**
 * Used for all editor inputs (keyboard shortcuts, ...)
 */
class FInputBindingSettingsViewerModule
	: public IModuleInterface
	, public ISettingsViewer
{
public:
	static const FLazyName SettingsTabName;
	static const FLazyName ContainerName;

	//~ Begin ISettingsViewer
	virtual void ShowSettings(const FName& InCategoryName, const FName& InSectionName) override;
	//~ End ISettingsViewer

	//~ Begin IModuleInterface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
	//~ End IModuleInterface

private:
	TSharedRef<SDockTab> HandleSpawnSettingsTab(const FSpawnTabArgs& InSpawnTabArgs);

	TWeakPtr<ISettingsEditorModel> SettingsEditorModelWeak;
};
